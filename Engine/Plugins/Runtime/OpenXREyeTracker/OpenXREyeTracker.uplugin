{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "OpenXREyeTracker",
	"Description": "OpenXR Eye Tracker provides XR_EXT_eye_gaze_interaction support.",
	"Category": "Virtual Reality",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "OpenXREyeTracker",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit"
		}
	],
	"Plugins": [
		{
			"Name": "XRBase",
			"Enabled": true
		},
		{
			"Name": "OpenXR",
			"Enabled": true
		}
	]
}
