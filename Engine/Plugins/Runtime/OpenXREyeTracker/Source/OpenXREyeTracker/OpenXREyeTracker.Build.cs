// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
    public class OpenXREyeTracker : ModuleRules
    {
        public OpenXREyeTracker(ReadOnlyTargetRules Target) 
				: base(Target)
        {
			PublicDependencyModuleNames.Add("EyeTracker");

            PrivateDependencyModuleNames.AddRange(
                new string[]
                {
                    "Core",
                    "CoreUObject",
					"ApplicationCore",
                    "Engine",
                    "InputDevice",
                    "InputCore",
                    "HeadMountedDisplay",
                    "OpenXRHMD",
					"OpenXRInput",
					"XRBase"
				}
				);

            AddEngineThirdPartyPrivateStaticDependencies(Target, "OpenXR");

            if (Target.bBuildEditor == true)
            {
                PrivateDependencyModuleNames.Add("UnrealEd");
            }
        }
    }
}
