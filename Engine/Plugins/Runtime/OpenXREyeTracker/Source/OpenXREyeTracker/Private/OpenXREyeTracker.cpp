// Copyright Epic Games, Inc. All Rights Reserved.

#include "OpenXREyeTracker.h"
#include "EyeTrackerTypes.h"
#include "IXRTrackingSystem.h"
#include "GameFramework/HUD.h"
#include "OpenXRCore.h"
#include "DrawDebugHelpers.h"
#include "Engine/Engine.h"

#if WITH_EDITOR
#endif

IMPLEMENT_MODULE(FOpenXREyeTrackerModule, OpenXREyeTracker);

static TAutoConsoleVariable<int32> CVarEnableOpenXREyetrackingDebug(TEXT("OpenXR.debug.EnableEyetrackingDebug"), 1, TEXT("0 - Eyetracking debug visualizations are disabled. 1 - Eyetracking debug visualizations are enabled."));

FOpenXREyeTracker::FOpenXREyeTracker()
{
	RegisterOpenXRExtensionModularFeature();
}

FOpenXREyeTracker::~FOpenXREyeTracker()
{
	Destroy();
}

void FOpenXREyeTracker::Destroy()
{
}

bool FOpenXREyeTracker::GetRequiredExtensions(TArray<const ANSICHAR*>& OutExtensions)
{
	TArray<XrApiLayerProperties> Properties;
	EnumerateOpenXRApiLayers(Properties);

	// Some API layers can crash the loader when enabled, if they're present we shouldn't enable the extension
	for (const XrApiLayerProperties& Layer : Properties)
	{
		if (FCStringAnsi::Strstr(Layer.layerName, "XR_APILAYER_VIVE_eye_tracking") &&
			Layer.layerVersion <= 1)
		{
			return false;
		}
	}

	OutExtensions.Add(XR_EXT_EYE_GAZE_INTERACTION_EXTENSION_NAME);
	return true;
}

void FOpenXREyeTracker::PostCreateInstance(XrInstance InInstance)
{
	Instance = InInstance;
}

bool FOpenXREyeTracker::GetInteractionProfiles(XrInstance InInstance, TArray<FString>& OutKeyPrefixes, TArray<XrPath>& OutPaths, TArray<bool>& OutHasHaptics)
{
	OutKeyPrefixes.Add("EyeTracker");
	OutHasHaptics.Add(false);
	XrPath OutPath;
	XrResult Result = xrStringToPath(InInstance, "/interaction_profiles/ext/eye_gaze_interaction", &OutPath);
	OutPaths.Add(OutPath);
	return Result == XR_SUCCESS;
}

void FOpenXREyeTracker::AttachActionSets(TSet<XrActionSet>& OutActionSets)
{
	check(Instance != XR_NULL_HANDLE);

	// This is a bit of a pain right now.  Hopefully future refactors will make it better.
	// We are creating an action set for our eye tracking pose.  An action set can have session create/destroy
	// lifetime.  However currently the OpenXRInput module is loaded well after the OpenXRHMDModule creates 
	// the session, so we can't just setup all this input system right then.  OpenXRInput instead checks if it is live and if 
	// not destroys any existing sets and then creates new ones near xrBeginSession (where the session starts running).
	// It marks them as dead near xrDestroySession so that they will be destroyed on the BeginSession of the next created session, 
	// if any.
	//
	// To mirror that lifetime we are destroying and creating in AttachActionSets and marking as dead in OnDestroySession.
	//
	// Note the ActionSpace is easier because it is never used outside of this ExtensionPlugin.  We are creating it, if necessary, 
	// in OnBeginSession and destroying it in OnDestroySession.  If we had a good CreateSession hook we could create it then, along with
	// the ActionSet and Action.

	// We could have an action set from a previous session.  If so it needs to go away.
	if (EyeTrackerActionSet != XR_NULL_HANDLE)
	{
		xrDestroyActionSet(EyeTrackerActionSet);
		EyeTrackerActionSet = XR_NULL_HANDLE;
		EyeTrackerAction = XR_NULL_HANDLE;
	}

	{
		XrActionSetCreateInfo Info;
		Info.type = XR_TYPE_ACTION_SET_CREATE_INFO;
		Info.next = nullptr;
		FCStringAnsi::Strncpy(Info.actionSetName, "openxreyetrackeractionset", XR_MAX_ACTION_SET_NAME_SIZE);
		FCStringAnsi::Strncpy(Info.localizedActionSetName, "OpenXR Eye Tracker Action Set", XR_MAX_LOCALIZED_ACTION_SET_NAME_SIZE);
		Info.priority = 0;
		XR_ENSURE(xrCreateActionSet(Instance, &Info, &EyeTrackerActionSet));
	}

	{
		check(EyeTrackerAction == XR_NULL_HANDLE);
		XrActionCreateInfo Info;
		Info.type = XR_TYPE_ACTION_CREATE_INFO;
		Info.next = nullptr;
		Info.countSubactionPaths = 0;
		Info.subactionPaths = nullptr;
		FCStringAnsi::Strncpy(Info.actionName, "openxreyetrackeraction", XR_MAX_ACTION_NAME_SIZE);
		FCStringAnsi::Strncpy(Info.localizedActionName, "OpenXR Eye Tracker Action", XR_MAX_LOCALIZED_ACTION_NAME_SIZE);
		Info.actionType = XR_ACTION_TYPE_POSE_INPUT;
		XR_ENSURE(xrCreateAction(EyeTrackerActionSet, &Info, &EyeTrackerAction));
	}

	{
		XrPath EyeGazeInteractionProfilePath = XR_NULL_PATH;
		XR_ENSURE(xrStringToPath(Instance, "/interaction_profiles/ext/eye_gaze_interaction", &EyeGazeInteractionProfilePath));

		XrPath GazePosePath = XR_NULL_PATH;
		XR_ENSURE(xrStringToPath(Instance, "/user/eyes_ext/input/gaze_ext/pose", &GazePosePath));

		XrActionSuggestedBinding Bindings;
		Bindings.action = EyeTrackerAction;
		Bindings.binding = GazePosePath;

		XrInteractionProfileSuggestedBinding SuggestedBindings{ XR_TYPE_INTERACTION_PROFILE_SUGGESTED_BINDING };
		SuggestedBindings.interactionProfile = EyeGazeInteractionProfilePath;
		SuggestedBindings.suggestedBindings = &Bindings;
		SuggestedBindings.countSuggestedBindings = 1;
		XR_ENSURE(xrSuggestInteractionProfileBindings(Instance, &SuggestedBindings));
	}

	OutActionSets.Add(EyeTrackerActionSet);
}

const void* FOpenXREyeTracker::OnBeginSession(XrSession InSession, const void* InNext)
{
	static FName SystemName(TEXT("OpenXR"));
	if (GEngine->XRSystem.IsValid() && (GEngine->XRSystem->GetSystemName() == SystemName))
	{
		XRTrackingSystem = GEngine->XRSystem.Get();
	}

	if (GazeActionSpace == XR_NULL_HANDLE)
	{
		GazeActionSpace = XR_NULL_HANDLE;
		XrActionSpaceCreateInfo CreateActionSpaceInfo{ XR_TYPE_ACTION_SPACE_CREATE_INFO };
		check(EyeTrackerAction != XR_NULL_HANDLE);
		CreateActionSpaceInfo.action = EyeTrackerAction;
		CreateActionSpaceInfo.poseInActionSpace = ToXrPose(FTransform::Identity);
		XR_ENSURE(xrCreateActionSpace(InSession, &CreateActionSpaceInfo, &GazeActionSpace));

		SyncInfo.countActiveActionSets = 0;
		SyncInfo.activeActionSets = XR_NULL_HANDLE;
	}

	bSessionStarted = true;

	return InNext;
}

void FOpenXREyeTracker::OnDestroySession(XrSession InSession)
{
	if (GazeActionSpace)
	{
		XR_ENSURE(xrDestroySpace(GazeActionSpace));
	}
	GazeActionSpace = XR_NULL_HANDLE;
}

void FOpenXREyeTracker::GetActiveActionSetsForSync(TArray<XrActiveActionSet>& OutActiveSets)
{
	check(EyeTrackerActionSet != XR_NULL_HANDLE);

	OutActiveSets.Add(XrActiveActionSet{ EyeTrackerActionSet, XR_NULL_PATH });
}

void FOpenXREyeTracker::PostSyncActions(XrSession InSession)
{
	check(EyeTrackerAction != XR_NULL_HANDLE);

	XrActionStateGetInfo GetActionStateInfo{ XR_TYPE_ACTION_STATE_GET_INFO };
	GetActionStateInfo.action = EyeTrackerAction;
	XR_ENSURE(xrGetActionStatePose(InSession, &GetActionStateInfo, &ActionStatePose));
}

void FOpenXREyeTracker::UpdateDeviceLocations(XrSession InSession, XrTime DisplayTime, XrSpace TrackingSpace)
{
	if (ActionStatePose.isActive) 
	{
		check(GazeActionSpace != XR_NULL_HANDLE);
		XR_ENSURE(xrLocateSpace(GazeActionSpace, TrackingSpace, DisplayTime, &EyeTrackerSpaceLocation));
	}
	else
	{
		// Clear the tracked bits if the action is not active
		const XrSpaceLocationFlags TrackedFlags = XR_SPACE_LOCATION_ORIENTATION_TRACKED_BIT | XR_SPACE_LOCATION_POSITION_TRACKED_BIT;
		EyeTrackerSpaceLocation.locationFlags &= ~TrackedFlags;
	}
}

bool FOpenXREyeTracker::GetEyeTrackerGazeData(FEyeTrackerGazeData& OutGazeData) const
{
	if (!bSessionStarted)
	{
		OutGazeData = FEyeTrackerGazeData();
		return false;
	}

	const XrSpaceLocationFlags ValidFlags = XR_SPACE_LOCATION_ORIENTATION_VALID_BIT | XR_SPACE_LOCATION_POSITION_VALID_BIT;
	const XrSpaceLocationFlags TrackedFlags = XR_SPACE_LOCATION_ORIENTATION_TRACKED_BIT | XR_SPACE_LOCATION_POSITION_TRACKED_BIT;

	if ((EyeTrackerSpaceLocation.locationFlags & ValidFlags) != ValidFlags)
	{
		// Either Orientation or position are invalid
		OutGazeData = FEyeTrackerGazeData();
		return false;
	}
	else if ((EyeTrackerSpaceLocation.locationFlags & TrackedFlags) != TrackedFlags)
	{
		// Orientation and/or position are old or an estimate of some kind, confidence is low.
		OutGazeData.ConfidenceValue = 0.0f;
	}
	else
	{
		// Both orientation and position are fully tracked now, confidence is high.
		OutGazeData.ConfidenceValue = 1.0f;
	}

	const float WorldToMetersScale = XRTrackingSystem->GetWorldToMetersScale();
	const XrPosef& Pose = EyeTrackerSpaceLocation.pose;
	const FTransform EyeTrackerTransform = ToFTransform(Pose, WorldToMetersScale);

	const FTransform& TrackingToWoldTransform = XRTrackingSystem->GetTrackingToWorldTransform();
	const FTransform EyeTransform = EyeTrackerTransform * TrackingToWoldTransform;

	OutGazeData.GazeDirection = EyeTransform.TransformVector(FVector::ForwardVector);
	OutGazeData.GazeOrigin = EyeTransform.GetLocation();
	OutGazeData.FixationPoint = FVector::ZeroVector; //not supported

	return true;
}

bool FOpenXREyeTracker::GetEyeTrackerStereoGazeData(FEyeTrackerStereoGazeData& OutStereoGazeData) const
{
	OutStereoGazeData = FEyeTrackerStereoGazeData();
	return false;
}

EEyeTrackerStatus FOpenXREyeTracker::GetEyeTrackerStatus() const
{
	if (!bSessionStarted)
	{
		return EEyeTrackerStatus::NotConnected;
	}

	const XrSpaceLocationFlags ValidFlags = XR_SPACE_LOCATION_ORIENTATION_VALID_BIT | XR_SPACE_LOCATION_POSITION_VALID_BIT;
	const XrSpaceLocationFlags TrackedFlags = XR_SPACE_LOCATION_ORIENTATION_TRACKED_BIT | XR_SPACE_LOCATION_POSITION_TRACKED_BIT;

	if ((EyeTrackerSpaceLocation.locationFlags & ValidFlags) != ValidFlags)
	{
		return EEyeTrackerStatus::NotTracking;
	}

	if ((EyeTrackerSpaceLocation.locationFlags & TrackedFlags) != TrackedFlags)
	{
		return EEyeTrackerStatus::NotTracking;
	}
	else
	{
		return EEyeTrackerStatus::Tracking;
	}
}

bool FOpenXREyeTracker::IsStereoGazeDataAvailable() const
{
	return false;
}

void FOpenXREyeTracker::DrawDebug(AHUD* HUD, UCanvas* Canvas, const FDebugDisplayInfo& DisplayInfo, float& YL, float& YPos)
{
	if (!bSessionStarted)
	{
		return;
	}

	const XrSpaceLocationFlags ValidFlags = XR_SPACE_LOCATION_ORIENTATION_VALID_BIT | XR_SPACE_LOCATION_POSITION_VALID_BIT;
	const XrSpaceLocationFlags TrackedFlags = XR_SPACE_LOCATION_ORIENTATION_TRACKED_BIT | XR_SPACE_LOCATION_POSITION_TRACKED_BIT;

	if ((EyeTrackerSpaceLocation.locationFlags & ValidFlags) != ValidFlags)
	{
		return;
	}

	FColor DrawColor = FColor::Yellow;
	if ((EyeTrackerSpaceLocation.locationFlags & TrackedFlags) == TrackedFlags)
	{
		DrawColor = FColor::Green;
	}

	const float WorldToMetersScale = XRTrackingSystem->GetWorldToMetersScale();
	const XrPosef& Pose = EyeTrackerSpaceLocation.pose;
	FTransform EyeTrackerTransform = ToFTransform(Pose, WorldToMetersScale);

	FVector GazeDirection = EyeTrackerTransform.TransformVector(FVector::ForwardVector);
	FVector GazeOrigin = EyeTrackerTransform.GetLocation();
	FVector DebugPos = GazeOrigin + (GazeDirection * 100.0f);
	DrawDebugSphere(HUD->GetWorld(), DebugPos, 20.0f, 16, DrawColor);
}

/************************************************************************/
/* FOpenXREyeTrackerModule                                           */
/************************************************************************/

FOpenXREyeTrackerModule::FOpenXREyeTrackerModule()
{
}

void FOpenXREyeTrackerModule::StartupModule()
{
	IEyeTrackerModule::StartupModule();
	EyeTracker = TSharedPtr<FOpenXREyeTracker, ESPMode::ThreadSafe>(new FOpenXREyeTracker());
	OnDrawDebugHandle = AHUD::OnShowDebugInfo.AddRaw(this, &FOpenXREyeTrackerModule::OnDrawDebug);
}

void FOpenXREyeTrackerModule::ShutdownModule()
{
	AHUD::OnShowDebugInfo.Remove(OnDrawDebugHandle);
}

TSharedPtr<class IEyeTracker, ESPMode::ThreadSafe> FOpenXREyeTrackerModule::CreateEyeTracker()
{
	return EyeTracker;
}

void FOpenXREyeTrackerModule::OnDrawDebug(AHUD* HUD, UCanvas* Canvas, const FDebugDisplayInfo& DisplayInfo, float& YL, float& YPos)
{
	if (CVarEnableOpenXREyetrackingDebug.GetValueOnGameThread())
	{
		if (EyeTracker.IsValid())
		{
			EyeTracker->DrawDebug(HUD, Canvas, DisplayInfo, YL, YPos);
		}
	}
}

bool FOpenXREyeTrackerModule::IsEyeTrackerConnected() const
{
	if (EyeTracker.IsValid())
	{
		EEyeTrackerStatus Status = EyeTracker->GetEyeTrackerStatus();
		if ((Status != EEyeTrackerStatus::NotTracking) && (Status != EEyeTrackerStatus::NotConnected))
		{
			return true;
		}
	}

	return false;
}
