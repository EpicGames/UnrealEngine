// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "IOpenXREyeTrackerModule.h"
#include "IEyeTracker.h"
#include "IOpenXRExtensionPlugin.h"


class AHUD;
class FDebugDisplayInfo;
class UCanvas;

class FOpenXREyeTracker : public IEyeTracker, public IOpenXRExtensionPlugin
{
public:
	FOpenXREyeTracker();
	virtual ~FOpenXREyeTracker();

	/************************************************************************/
	/* IEyeTracker                                                          */
	/************************************************************************/
	void Destroy();
	virtual void SetEyeTrackedPlayer(APlayerController* PlayerController) override {};
	virtual bool GetEyeTrackerGazeData(FEyeTrackerGazeData& OutGazeData) const override;
	virtual bool GetEyeTrackerStereoGazeData(FEyeTrackerStereoGazeData& OutGazeData) const override;
	virtual EEyeTrackerStatus GetEyeTrackerStatus() const override;
	virtual bool IsStereoGazeDataAvailable() const override;
	
	/************************************************************************/
	/* IOpenXRExtensionPlugin                                               */
	/************************************************************************/

	virtual FString GetDisplayName() override
	{
		return FString(TEXT("OpenXREyeTracker"));
	}
	virtual bool GetRequiredExtensions(TArray<const ANSICHAR*>& OutExtensions) override;
	virtual void PostCreateInstance(XrInstance InInstance) override;
	virtual bool GetInteractionProfiles(XrInstance InInstance, TArray<FString>& OutKeyPrefixes, TArray<XrPath>& OutPaths, TArray<bool>& OutHasHaptics) override;
	virtual void AttachActionSets(TSet<XrActionSet>& OutActionSets) override;
	virtual const void* OnBeginSession(XrSession InSession, const void* InNext) override;
	virtual void OnDestroySession(XrSession InSession) override;
	virtual void GetActiveActionSetsForSync(TArray<XrActiveActionSet>& OutActiveSets) override;
	virtual void PostSyncActions(XrSession InSession) override;
	virtual void UpdateDeviceLocations(XrSession InSession, XrTime DisplayTime, XrSpace TrackingSpace) override;

	void DrawDebug(AHUD* HUD, UCanvas* Canvas, const FDebugDisplayInfo& DisplayInfo, float& YL, float& YPos);

private:
	class IXRTrackingSystem* XRTrackingSystem = nullptr;
	XrInstance Instance = XR_NULL_HANDLE;
	bool bSessionStarted = false;
	XrActionsSyncInfo SyncInfo{ XR_TYPE_ACTIONS_SYNC_INFO };
	XrAction EyeTrackerAction = XR_NULL_HANDLE;
	XrActionSet EyeTrackerActionSet = XR_NULL_HANDLE;
	XrSpace GazeActionSpace = XR_NULL_HANDLE;
	XrActionStatePose ActionStatePose{ XR_TYPE_ACTION_STATE_POSE };

	// EyeTracker cached data
	XrSpaceLocation EyeTrackerSpaceLocation{ XR_TYPE_SPACE_LOCATION };
};

class FOpenXREyeTrackerModule : public IOpenXREyeTrackerModule
{
	/************************************************************************/
	/* IInputDeviceModule                                                   */
	/************************************************************************/
public:
	FOpenXREyeTrackerModule();
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	virtual TSharedPtr< class IEyeTracker, ESPMode::ThreadSafe > CreateEyeTracker() override;

	/************************************************************************/
	/* IEyeTrackerModule													*/
	/************************************************************************/

	virtual bool IsEyeTrackerConnected() const override;

private:
	TSharedPtr<FOpenXREyeTracker, ESPMode::ThreadSafe> EyeTracker;
	FDelegateHandle OnDrawDebugHandle;

	void OnDrawDebug(AHUD* HUD, UCanvas* Canvas, const FDebugDisplayInfo& DisplayInfo, float& YL, float& YPos);
};
